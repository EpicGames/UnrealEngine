<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <ApplicationManifest>app.manifest</ApplicationManifest>
    <RootNamespace>AutomationToolLauncher</RootNamespace>
    <AssemblyName>AutomationToolLauncher</AssemblyName>
    <OutputPath>..\..\..\Binaries\DotNET\AutomationToolLauncher</OutputPath>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
  </ItemGroup>

  <!-- Strip transitive dependencies not reachable from this launcher's code paths -->
  <Target Name="RemoveUnneededTransitiveDependencies" AfterTargets="ResolveReferences">
    <ItemGroup>
      <!-- EpicGames.IoHash and its native Blake3 binaries (pulled in by EpicGames.Build) -->
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'EpicGames.IoHash'" />
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'Blake3'" />
      <!-- EpicGames.MsBuild and Microsoft.Build.Locator (pulled in by EpicGames.Build) -->
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'EpicGames.MsBuild'" />
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'Microsoft.Build.Locator'" />
      <!-- JetBrains.Annotations, OpenTracing and Polly (pulled in by EpicGames.Core) -->
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'JetBrains.Annotations'" />
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'OpenTracing'" />
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'Polly'" />
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="'%(Filename)' == 'Polly.Extensions.Http'" />
      <!-- Microsoft.Extensions.* (pulled in by EpicGames.Core) -->
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="$([System.String]::Copy('%(Filename)').StartsWith('Microsoft.Extensions.'))" />
      <!-- Native runtime folders (blake3_dotnet.dll etc.) -->
      <ReferenceCopyLocalPaths Remove="@(ReferenceCopyLocalPaths)" Condition="$([System.String]::Copy('%(Identity)').Contains('\runtimes\'))" />
    </ItemGroup>
  </Target>
</Project>