<Solution>
  <Configurations>
    <BuildType Name="Analyze" />
    <BuildType Name="Debug" />
    <BuildType Name="Development" />
    <BuildType Name="Release" />
  </Configurations>
  <Folder Name="/Shared/">
    <Project Path="../Shared/EpicGames.Build/EpicGames.Build.csproj" />
    <Project Path="../Shared/EpicGames.Core/EpicGames.Core.csproj" />
    <Project Path="../Shared/EpicGames.IoHash/EpicGames.IoHash.csproj" />
  </Folder>
  <Project Path="AutomationToolLauncher.csproj" />
</Solution>
