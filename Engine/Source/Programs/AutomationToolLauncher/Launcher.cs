// Copyright Epic Games, Inc. All Rights Reserved.
using EpicGames.Core;
using System;
using System.Diagnostics;
using System.Threading.Tasks;
using UnrealBuildBase;

namespace AutomationToolLauncher;

class Launcher
{
	static async Task<int> Main(string[] arguments)
	{
		FileReference uatPath = GetUatPath();

		if (!FileReference.Exists(uatPath))
		{
			Console.WriteLine($"AutomationTool does not exist at: {uatPath}");
			return -1;
		}

		try
		{
			ProcessStartInfo psi = new(Unreal.DotnetPath.FullName);
			psi.ArgumentList.Add(uatPath.FullName);
			foreach (string arg in arguments)
			{
				psi.ArgumentList.Add(arg);
			}
			using Process uatProcess = Process.Start(psi) ?? throw new InvalidOperationException("Failed to start AutomationTool process.");
			await uatProcess.WaitForExitAsync();
			return uatProcess.ExitCode;
		}
		catch (Exception ex)
		{
			Console.WriteLine(ex.Message);
			Console.WriteLine(ex.StackTrace);
		}

		return -1;
	}

	static FileReference GetUatPath()
	{
		DirectoryReference applicationBase = new DirectoryReference(AppContext.BaseDirectory);
		return FileReference.Combine(applicationBase, "..", "AutomationTool", "AutomationTool.dll");
	}
}
