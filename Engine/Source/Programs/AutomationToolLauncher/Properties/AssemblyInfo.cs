// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;

[assembly: AssemblyTitle("AutomationToolLauncher")]
