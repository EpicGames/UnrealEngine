<?xml version="1.0"?>
<configuration>
	<startup>
		<supportedRuntime version="v4.0" sku=".NETFramework,Version=v4.6.2"/>
	</startup>
	<runtime>
		<assemblyBinding xmlns="urn:schemas-microsoft-com:asm.v1">
			<dependentAssembly>
				<assemblyIdentity name="Newtonsoft.Json" publicKeyToken="30ad4fe6b2a6aeed" culture="neutral"/>
				<bindingRedirect oldVersion="0.0.0.0-8.0.0.0" newVersion="9.0.0.0"/>
			</dependentAssembly>
		</assemblyBinding>
	</runtime>
</configuration>
